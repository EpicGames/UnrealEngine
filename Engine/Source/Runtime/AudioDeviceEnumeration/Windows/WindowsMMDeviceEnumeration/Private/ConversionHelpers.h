// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#include "AudioMixer.h"	
#include "Windows/AllowWindowsPlatformTypes.h"

THIRD_PARTY_INCLUDES_START
#include <mmdeviceapi.h>			// IMMNotificationClient
#include <audiopolicy.h>			// IAudioSessionEvents
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformTypes.h"

namespace Audio
{
	static inline EAudioDeviceState ConvertWordToDeviceState(DWORD InWord)
	{
		switch (InWord)
		{
		case DEVICE_STATE_ACTIVE: return Audio::EAudioDeviceState::Active;
		case DEVICE_STATE_DISABLED: return Audio::EAudioDeviceState::Disabled;
		case DEVICE_STATE_UNPLUGGED: return Audio::EAudioDeviceState::Unplugged;
		case DEVICE_STATE_NOTPRESENT: return Audio::EAudioDeviceState::NotPresent;
		default:
			break;
		}
		checkNoEntry();
		return Audio::EAudioDeviceState::NotPresent;
	}

	static inline IAudioMixerDeviceChangedListener::EDisconnectReason AudioSessionDisconnectToEDisconnectReason(AudioSessionDisconnectReason InDisconnectReason)
	{
		using namespace Audio;
		switch (InDisconnectReason)
		{
		case DisconnectReasonDeviceRemoval:			return IAudioMixerDeviceChangedListener::EDisconnectReason::DeviceRemoval;
		case DisconnectReasonServerShutdown:		return IAudioMixerDeviceChangedListener::EDisconnectReason::ServerShutdown;
		case DisconnectReasonFormatChanged:			return IAudioMixerDeviceChangedListener::EDisconnectReason::FormatChanged;
		case DisconnectReasonSessionLogoff:			return IAudioMixerDeviceChangedListener::EDisconnectReason::SessionLogoff;
		case DisconnectReasonSessionDisconnected:	return IAudioMixerDeviceChangedListener::EDisconnectReason::SessionDisconnected;
		case DisconnectReasonExclusiveModeOverride:	return IAudioMixerDeviceChangedListener::EDisconnectReason::ExclusiveModeOverride;
		default: break;
		}

		checkNoEntry();
		return IAudioMixerDeviceChangedListener::EDisconnectReason::DeviceRemoval;
	}

}
