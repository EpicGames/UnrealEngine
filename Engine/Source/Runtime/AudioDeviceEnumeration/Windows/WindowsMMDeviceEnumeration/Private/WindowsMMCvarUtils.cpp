// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsMMCvarUtils.h"
#include "HAL/IConsoleManager.h"

static int32 DisableDeviceSwapCVar = 0;
FAutoConsoleVariableRef CVarDisableDeviceSwap(
	TEXT("au.DisableDeviceSwap"),
	DisableDeviceSwapCVar,
	TEXT("Disable device swap handling code for Audio Mixer on Windows.\n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

static int32 EnableDetailedWindowsDeviceLoggingCVar = 0;
FAutoConsoleVariableRef CVarEnableDetailedWindowsDeviceLogging(
	TEXT("au.EnableDetailedWindowsDeviceLogging"),
	EnableDetailedWindowsDeviceLoggingCVar,
	TEXT("Enables detailed windows device logging.\n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

static int32 EnableAggregateAudioDevicesCVar = 0;
FAutoConsoleVariableRef CVarEnableAggregateAudioDevices(
	TEXT("au.Wasapi.EnableAggregateAudioDevices"),
	EnableAggregateAudioDevicesCVar,
	TEXT("Enables WASAPI aggregate audio devices.\n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

namespace Audio
{
	bool WindowsMMCvarUtils::ShouldIgnoreDeviceSwaps()
	{
		return DisableDeviceSwapCVar != 0;
	}

	bool WindowsMMCvarUtils::ShouldLogDeviceSwaps()
	{
		return EnableDetailedWindowsDeviceLoggingCVar != 0;
	}

	bool WindowsMMCvarUtils::IsAggregateDeviceSupportCVarEnabled()
	{
		return EnableAggregateAudioDevicesCVar != 0;
	}
} // namespace Audio
