// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


namespace Audio
{
	class WindowsMMCvarUtils
	{
	public:
		/** Called by FWindowsMMNotificationClient to bypass notifications for audio device changes: */
		static bool ShouldIgnoreDeviceSwaps();

		/** Called by FWindowsMMNotificationClient to toggle logging for audio device changes: */
		static bool ShouldLogDeviceSwaps();

		/** Called by FWindowsMMNotificationClient to determine if Wasapi aggregate device support is enabled. */
		static bool IsAggregateDeviceSupportCVarEnabled();
	};

} // namespace Audio