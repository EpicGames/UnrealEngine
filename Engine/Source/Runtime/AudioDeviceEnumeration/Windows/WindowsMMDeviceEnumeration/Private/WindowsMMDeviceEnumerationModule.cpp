// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "HAL/Platform.h"

#include "Windows/AllowWindowsPlatformTypes.h"

THIRD_PARTY_INCLUDES_START
// Linkage to define  Windows PKEY GUIDs included by Notification/DeviceInfoCache, otherwise they are unresolved extern.
#include <initguid.h>
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformTypes.h"

#include "WindowsMMNotificationClient.h"
#include "WindowsMMDeviceInfoCache.h"
#include "WindowsMMDeviceEnumerationLog.h"

DEFINE_LOG_CATEGORY(LogAudioEnumeration);

namespace Audio
{
	class FWindowsMMDeviceEnumerationModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override
		{
		}

		virtual void ShutdownModule() override
		{
		}
	};
}

IMPLEMENT_MODULE(Audio::FWindowsMMDeviceEnumerationModule, WindowsMMDeviceEnumeration)
