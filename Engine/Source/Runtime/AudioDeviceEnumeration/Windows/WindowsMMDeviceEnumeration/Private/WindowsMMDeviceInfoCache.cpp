// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsMMDeviceInfoCache.h"

#include "ConversionHelpers.h"
#include "HAL/IConsoleManager.h"
#include "Internationalization/Regex.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"

THIRD_PARTY_INCLUDES_START
#include <devicetopology.h>
#include <functiondiscoverykeys_devpkey.h>
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"

#include "WindowsMMStringUtils.h"
#include "Misc/ScopeRWLock.h"
#include "WindowsMMCvarUtils.h"
#include "WindowsMMDeviceEnumerationLog.h"

namespace Audio
{
	// Support for forcing the audio renderer on windows machines to initialize as if it was connected to a 7.1 output device.
	// This is useful for cases where a developer does not have access to a 7.1 device (or is working on a cloud machine) and 
	// wants to validate and excersie surround sound audio rendering code.
	static bool GetForceSurroundSound()
	{	
		static bool bForceSurroundSound = FParse::Param(FCommandLine::Get(), TEXT("ForceSurroundSound"));
		return bForceSurroundSound;
	}
	
	FWindowsMMDeviceCache::FCacheEntry& FWindowsMMDeviceCache::FCacheEntry::operator=(FCacheEntry&& InOther)
	{
		DeviceId = MoveTemp(InOther.DeviceId);
		FriendlyName = MoveTemp(InOther.FriendlyName);
		DeviceFriendlyName = MoveTemp(InOther.DeviceFriendlyName);
		State = MoveTemp(InOther.State);
		NumChannels = MoveTemp(InOther.NumChannels);
		SampleRate = MoveTemp(InOther.SampleRate);
		Type = MoveTemp(InOther.Type);
		ChannelBitmask = MoveTemp(InOther.ChannelBitmask);
		OutputChannels = MoveTemp(InOther.OutputChannels);
		HardwareId = MoveTemp(InOther.HardwareId);
		FilterId = MoveTemp(InOther.FilterId);
		return *this;
	}

	FWindowsMMDeviceCache::FCacheEntry& FWindowsMMDeviceCache::FCacheEntry::operator=(const FCacheEntry& InOther)
	{
		// Copy everything but the lock. 
		DeviceId = InOther.DeviceId;
		FriendlyName = InOther.FriendlyName;
		DeviceFriendlyName = InOther.DeviceFriendlyName;
		State = InOther.State;
		NumChannels = InOther.NumChannels;
		SampleRate = InOther.SampleRate;
		Type = InOther.Type;
		ChannelBitmask = InOther.ChannelBitmask;
		OutputChannels = InOther.OutputChannels;
		HardwareId = InOther.HardwareId;
		FilterId = InOther.FilterId;
		return *this;
	}

	FWindowsMMDeviceCache::FCacheEntry::FCacheEntry(const FString& InDeviceId)
		: DeviceId{ InDeviceId }
	{
	}

	FWindowsMMDeviceCache::FCacheEntry::FCacheEntry(FCacheEntry&& InOther)
	{
		*this = MoveTemp(InOther);
	}

	FWindowsMMDeviceCache::FCacheEntry::FCacheEntry(const FCacheEntry& InOther)
	{
		*this = InOther;
	}

	FWindowsMMDeviceCache::FWindowsMMDeviceCache(bool bInEnableAggregateDeviceSupport) :
		bIsAggregateDeviceSupportEnabled(bInEnableAggregateDeviceSupport)
	{
		ensure(SUCCEEDED(CoCreateInstance(__uuidof(MMDeviceEnumerator), NULL, CLSCTX_INPROC_SERVER, IID_PPV_ARGS(&DeviceEnumerator))) && DeviceEnumerator);
		
		Refresh();
	}

	FWindowsMMDeviceCache::FWindowsMMDeviceCache() :
		FWindowsMMDeviceCache(false)
	{
	}

	void FWindowsMMDeviceCache::Refresh()
	{
		UE_LOGF(LogAudioEnumeration, Display, "Refreshing FWindowsMMDeviceCache");
		// Hold the write lock across both phases so a reader landing between
		// the two cannot observe a fully repopulated Cache map paired with
		// stale DefaultRenderId / DefaultCaptureId arrays.
		FWriteScopeLock WriteLock(CacheMutationLock);
		EnumerateEndpoints_NoLock();
		EnumerateDefaults_NoLock();
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FWindowsMMDeviceCache::EnumerateEndpoints()
	{
		// Deprecated wrapper: takes the lock then forwards to the no-lock implementation.
		FWriteScopeLock WriteLock(CacheMutationLock);
		EnumerateEndpoints_NoLock();
	}

	void FWindowsMMDeviceCache::EnumerateDefaults()
	{
		// Deprecated wrapper: takes the lock then forwards to the no-lock implementation.
		FWriteScopeLock WriteLock(CacheMutationLock);
		EnumerateDefaults_NoLock();
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool FWindowsMMDeviceCache::EnumerateChannelMask(uint32 InMask, FCacheEntry& OutInfo)
	{
		// Loop through the extensible format channel flags in the standard order and build our output channel array
		// From https://msdn.microsoft.com/en-us/library/windows/hardware/dn653308(v=vs.85).aspx
		// The channels in the interleaved stream corresponding to these spatial positions must appear in the order specified above. This holds true even in the 
		// case of a non-contiguous subset of channels. For example, if a stream contains left, bass enhance and right, then channel 1 is left, channel 2 is right, 
		// and channel 3 is bass enhance. This enables the linkage of multi-channel streams to well-defined multi-speaker configurations.

		static const uint32 REMOVE_ME_ChannelTypeMap[EAudioMixerChannel::ChannelTypeCount] =
		{
			SPEAKER_FRONT_LEFT,
			SPEAKER_FRONT_RIGHT,
			SPEAKER_FRONT_CENTER,
			SPEAKER_LOW_FREQUENCY,
			SPEAKER_BACK_LEFT,
			SPEAKER_BACK_RIGHT,
			SPEAKER_FRONT_LEFT_OF_CENTER,
			SPEAKER_FRONT_RIGHT_OF_CENTER,
			SPEAKER_BACK_CENTER,
			SPEAKER_SIDE_LEFT,
			SPEAKER_SIDE_RIGHT,
			SPEAKER_TOP_CENTER,
			SPEAKER_TOP_FRONT_LEFT,
			SPEAKER_TOP_FRONT_CENTER,
			SPEAKER_TOP_FRONT_RIGHT,
			SPEAKER_TOP_BACK_LEFT,
			SPEAKER_TOP_BACK_CENTER,
			SPEAKER_TOP_BACK_RIGHT,
			SPEAKER_RESERVED,
		};

		OutInfo.ChannelBitmask = InMask;
		OutInfo.OutputChannels.Reset();

		// No need to enumerate speakers for capture devices.
		if (OutInfo.Type == EDeviceEndpointType::Capture)
		{
			return true;
		}

		uint32 ChanCount = 0;
		for (uint32 ChannelTypeIndex = 0; ChannelTypeIndex < EAudioMixerChannel::ChannelTypeCount && ChanCount < (uint32)OutInfo.NumChannels; ++ChannelTypeIndex)
		{
			if (InMask & REMOVE_ME_ChannelTypeMap[ChannelTypeIndex])
			{
				OutInfo.OutputChannels.Add((EAudioMixerChannel::Type)ChannelTypeIndex);
				++ChanCount;
			}
		}

		// We didn't match channel masks for all channels, revert to a default ordering
		if (ChanCount < (uint32)OutInfo.NumChannels)
		{
			UE_CLOGF(WindowsMMCvarUtils::ShouldLogDeviceSwaps(), LogAudioEnumeration, Warning, "FWindowsMMDeviceCache: Did not find the channel type flags for audio device '%ls'. Reverting to a default channel ordering.", *OutInfo.FriendlyName);

			OutInfo.OutputChannels.Reset();

			static const EAudioMixerChannel::Type DefaultChannelOrdering[] = {
				EAudioMixerChannel::FrontLeft,
				EAudioMixerChannel::FrontRight,
				EAudioMixerChannel::FrontCenter,
				EAudioMixerChannel::LowFrequency,
				EAudioMixerChannel::SideLeft,
				EAudioMixerChannel::SideRight,
				EAudioMixerChannel::BackLeft,
				EAudioMixerChannel::BackRight,
			};

			const EAudioMixerChannel::Type* ChannelOrdering = DefaultChannelOrdering;

			// Override channel ordering for some special cases
			if (OutInfo.NumChannels == 4)
			{
				static EAudioMixerChannel::Type DefaultChannelOrderingQuad[] = {
					EAudioMixerChannel::FrontLeft,
					EAudioMixerChannel::FrontRight,
					EAudioMixerChannel::BackLeft,
					EAudioMixerChannel::BackRight,
				};

				ChannelOrdering = DefaultChannelOrderingQuad;
			}
			else if (OutInfo.NumChannels == 6)
			{
				static const EAudioMixerChannel::Type DefaultChannelOrdering51[] = {
					EAudioMixerChannel::FrontLeft,
					EAudioMixerChannel::FrontRight,
					EAudioMixerChannel::FrontCenter,
					EAudioMixerChannel::LowFrequency,
					EAudioMixerChannel::BackLeft,
					EAudioMixerChannel::BackRight,
				};

				ChannelOrdering = DefaultChannelOrdering51;
			}

			check(OutInfo.NumChannels <= 8);
			for (int32 Index = 0; Index < OutInfo.NumChannels; ++Index)
			{
				OutInfo.OutputChannels.Add(ChannelOrdering[Index]);
			}
		}
		return true;
	}

	bool FWindowsMMDeviceCache::EnumerateChannelFormat(const WAVEFORMATEX* InFormat, FCacheEntry& OutInfo)
	{
		OutInfo.OutputChannels.Empty();

		// Extensible format supports surround sound so we need to parse the channel configuration to build our channel output array
		if (InFormat->wFormatTag == WAVE_FORMAT_EXTENSIBLE)
		{
			// Cast to the extensible format to get access to extensible data
			const WAVEFORMATEXTENSIBLE* WaveFormatExtensible = (const WAVEFORMATEXTENSIBLE*)(InFormat);
			return EnumerateChannelMask(WaveFormatExtensible->dwChannelMask, OutInfo);
		}
		else
		{
			// Non-extensible formats only support mono or stereo channel output
			OutInfo.OutputChannels.Add(EAudioMixerChannel::FrontLeft);
			if (OutInfo.NumChannels == 2)
			{
				OutInfo.OutputChannels.Add(EAudioMixerChannel::FrontRight);
			}
		}

		// Aways success for now.
		return true;
	}

	EDeviceEndpointType FWindowsMMDeviceCache::QueryDeviceDataFlow(const TComPtr<IMMDevice>& InDevice) const
	{
		TComPtr<IMMEndpoint> Endpoint;
		if (SUCCEEDED(InDevice->QueryInterface(IID_PPV_ARGS(&Endpoint))))
		{
			EDataFlow DataFlow = eRender;
			if (SUCCEEDED(Endpoint->GetDataFlow(&DataFlow)))
			{
				switch (DataFlow)
				{
				case eRender:
					return EDeviceEndpointType::Render;
				case eCapture:
					return EDeviceEndpointType::Capture;
				default:
					break;
				}
			}
		}
		return EDeviceEndpointType::Unknown;
	}

	bool FWindowsMMDeviceCache::EnumerateDeviceProps(const TComPtr<IMMDevice>& InDevice, FCacheEntry& OutInfo)
	{
		// Mark if this is a Render Device or Capture or Unknown.
		OutInfo.Type = QueryDeviceDataFlow(InDevice);

		// Also query the device state.
		DWORD DeviceState = DEVICE_STATE_NOTPRESENT;
		if (SUCCEEDED(InDevice->GetState(&DeviceState)))
		{
			OutInfo.State = ConvertWordToDeviceState(DeviceState);
		}

		TComPtr<IPropertyStore> PropertyStore;
		if (SUCCEEDED(InDevice->OpenPropertyStore(STGM_READ, &PropertyStore)))
		{
			// Friendly Name
			PROPVARIANT FriendlyName;
			PropVariantInit(&FriendlyName);
			if (SUCCEEDED(PropertyStore->GetValue(PKEY_Device_FriendlyName, &FriendlyName)) && FriendlyName.pwszVal)
			{
				OutInfo.FriendlyName = FString(FriendlyName.pwszVal);
				PropVariantClear(&FriendlyName);
			}

			auto EnumDeviceFormat = [this](const TComPtr<IPropertyStore>& InPropStore, REFPROPERTYKEY InKey, FCacheEntry& OutInfo) -> bool
			{
				// Device Format
				PROPVARIANT DeviceFormat;
				PropVariantInit(&DeviceFormat);

				if (SUCCEEDED(InPropStore->GetValue(InKey, &DeviceFormat)) && DeviceFormat.blob.pBlobData)
				{
					const WAVEFORMATEX* WaveFormatEx = (const WAVEFORMATEX*)(DeviceFormat.blob.pBlobData);
					if (GetForceSurroundSound())
					{
						OutInfo.NumChannels = 8;
					}
					else
					{
						OutInfo.NumChannels = FMath::Clamp((int32)WaveFormatEx->nChannels, 2, 8);
					}
					
					OutInfo.SampleRate = WaveFormatEx->nSamplesPerSec;

					EnumerateChannelFormat(WaveFormatEx, OutInfo);
					PropVariantClear(&DeviceFormat);
					return true;
				}
				return false;
			};

			if (EnumDeviceFormat(PropertyStore, PKEY_AudioEngine_DeviceFormat, OutInfo) ||
				EnumDeviceFormat(PropertyStore, PKEY_AudioEngine_OEMFormat, OutInfo))
			{
			}
			else
			{
				// Log a warning if this device is active as we failed to ask for a format
				UE_CLOGF(DeviceState == DEVICE_STATE_ACTIVE, LogAudioEnumeration, Warning, "FWindowsMMDeviceCache: Failed to get Format for active device '%ls'", *OutInfo.FriendlyName);
			}
		}

		// Aways success for now.
		return true;
	}

	bool FWindowsMMDeviceCache::EnumerateHardwareTopology(const TComPtr<IMMDevice>& InDevice, FCacheEntry& OutInfo)
	{
		SCOPED_NAMED_EVENT(FWindowsMMDeviceCache_EnumerateHardwareTopology, FColor::Blue);

	#if PLATFORM_WINDOWS

		if (!DeviceEnumerator)
		{
			return false;
		}

		TComPtr<IDeviceTopology> RenderEndpointTopology;
		HRESULT Result = InDevice->Activate(__uuidof(IDeviceTopology), CLSCTX_ALL, NULL, (void**)&RenderEndpointTopology);
		if (FAILED(Result))
		{
			UE_LOGF(LogAudioEnumeration, Verbose, "FWindowsMMDeviceCache::EnumerateHardwareTopology failed to fetch IDeviceTopology: 0x%x", Result);
			return false;
		}

		TComPtr<IConnector> Connector;
		Result = RenderEndpointTopology->GetConnector(0, &Connector);
		if (FAILED(Result))
		{
			UE_LOGF(LogAudioEnumeration, Verbose, "FWindowsMMDeviceCache::EnumerateHardwareTopology failed to fetch connector: 0x%x", Result);
			return false;
		}

		LPWSTR RenderFilterId = nullptr;
		Result = Connector->GetDeviceIdConnectedTo(&RenderFilterId);
		if (FAILED(Result) || !RenderFilterId)
		{
			UE_LOGF(LogAudioEnumeration, Verbose, "FWindowsMMDeviceCache::EnumerateHardwareTopology failed to fetch render filter Id: 0x%x", Result);
			return false;
		}

		FString TempFilterId(RenderFilterId);
		CoTaskMemFree(RenderFilterId);

		TComPtr<IMMDevice> RenderDevnode;
		Result = DeviceEnumerator->GetDevice(*TempFilterId, &RenderDevnode);
		if (FAILED(Result))
		{
			UE_LOGF(LogAudioEnumeration, Verbose, "FWindowsMMDeviceCache::EnumerateHardwareTopology failed to fetch render node: 0x%x", Result);
			return false;
		}

		TComPtr<IPropertyStore> PropertyStore;
		Result = RenderDevnode->OpenPropertyStore(STGM_READ, &PropertyStore);
		if (FAILED(Result))
		{
			UE_LOGF(LogAudioEnumeration, Verbose, "FWindowsMMDeviceCache::EnumerateHardwareTopology failed to open property store: 0x%x", Result);
			return false;
		}

		PROPVARIANT HardwareId;
		PropVariantInit(&HardwareId);
		Result = PropertyStore->GetValue(PKEY_Device_InstanceId, &HardwareId);
		if (FAILED(Result) || !HardwareId.pwszVal)
		{
			UE_LOGF(LogAudioEnumeration, Verbose, "FWindowsMMDeviceCache::EnumerateHardwareTopology failed to fetch hardward Id: 0x%x", Result);
			return false;
		}

		OutInfo.HardwareId = FName(HardwareId.pwszVal);
		PropVariantClear(&HardwareId);

		OutInfo.FilterId = TempFilterId;

		return true;
	#else
		return false;
	#endif //PLATFORM_WINDOWS
	}

	void FWindowsMMDeviceCache::EnumerateEndpoints_NoLock()
	{
		SCOPED_NAMED_EVENT(FWindowsMMDeviceCache_EnumerateEndpoints, FColor::Blue);

		// Caller must hold CacheMutationLock for write.
		// Build a new cache from scratch.
		TMap<FName, FCacheEntry> NewCache;

		// Get Device Enumerator.
		if (DeviceEnumerator)
		{
			// Get Render Device Collection. (note we ask for ALL states, which include disabled/unplugged devices.).
			TComPtr<IMMDeviceCollection> DeviceCollection;
			uint32 DeviceCount = 0;
			if (SUCCEEDED(DeviceEnumerator->EnumAudioEndpoints(eAll, DEVICE_STATEMASK_ALL, &DeviceCollection)) && DeviceCollection &&
				SUCCEEDED(DeviceCollection->GetCount(&DeviceCount)))
			{
				for (uint32 i = 0; i < DeviceCount; ++i)
				{
					TComPtr<IMMDevice> Device;
					if (SUCCEEDED(DeviceCollection->Item(i, &Device)) && Device)
					{
						// Get the device id string (guid)
						Audio::FScopeComString DeviceIdString;
						if (SUCCEEDED(Device->GetId(&DeviceIdString.StringPtr)) && DeviceIdString)
						{
							FCacheEntry Info{ DeviceIdString.Get() };

							// Enumerate props into our info object.
							EnumerateDeviceProps(Device, Info);

							// Enumerate hardware topology to fetch hardware Id.
							if (IsAggregateDeviceSupportEnabled())
							{
								EnumerateHardwareTopology(Device, Info);
							}

							UE_LOGF(LogAudioEnumeration, Verbose, "FWindowsMMDeviceCache: %ls Device '%ls' ID='%ls'",
								Info.Type == EDeviceEndpointType::Capture ? TEXT("Capture") :
								Info.Type == EDeviceEndpointType::Render ? TEXT("Render") :
								TEXT("UNKNOWN!"),
								*Info.DeviceId.ToString(),
								*Info.FriendlyName
							);

							check(!NewCache.Contains(Info.DeviceId));
							NewCache.Emplace(Info.DeviceId, Info);
						}
					}
				}
			}

			// Finally, Replace cache with new one. Caller already holds the write lock.
			Cache = MoveTemp(NewCache);
		}
	}

	void FWindowsMMDeviceCache::EnumerateDefaults_NoLock()
	{
		if (!DeviceEnumerator.IsValid())
		{
			UE_LOGF(LogAudioEnumeration, Warning, "FWindowsMMDeviceCache::EnumerateDefaults() NULL device enumerator");
			return;
		}
		
		auto GetDefaultDeviceID = [this](EDataFlow InDataFlow, ERole InRole, FName& OutDeviceId) -> bool
		{
			// Mark default device.
			bool bSuccess = false;
			TComPtr<IMMDevice> DefaultDevice;
			if (SUCCEEDED(DeviceEnumerator->GetDefaultAudioEndpoint(InDataFlow, InRole, &DefaultDevice)))
			{
				if (DefaultDevice.IsValid())
				{
					Audio::FScopeComString DeviceIdString;
					if (SUCCEEDED(DefaultDevice->GetId(&DeviceIdString.StringPtr)) && DeviceIdString)
					{
						OutDeviceId = DeviceIdString.Get();
						bSuccess = true;
					}
				}
				else
				{
					UE_LOGF(LogAudioEnumeration, Display, "FWindowsMMDeviceCache::EnumerateDefaults() NULL default device for InDataFlow: %d InRole: %d", InDataFlow, InRole);
				}
			}
			return bSuccess;
		};

		// Get defaults (render, capture). Caller must hold CacheMutationLock for write.
		static_assert((int32)EAudioDeviceRole::COUNT == ERole_enum_count, "EAudioDeviceRole should be the same as ERole");
		for (int32 i = 0; i < ERole_enum_count; ++i)
		{
			FName DeviceIdName;
			if (GetDefaultDeviceID(eRender, static_cast<ERole>(i), DeviceIdName))
			{
				UE_CLOGF(!DeviceIdName.IsNone(), LogAudioEnumeration, Display, "FWindowsMMDeviceCache: Default Render Role='%ls', Device='%ls'", ToString((EAudioDeviceRole)i), *GetFriendlyName(DeviceIdName));
				DefaultRenderId[i] = DeviceIdName;
			}
			if (GetDefaultDeviceID(eCapture, static_cast<ERole>(i), DeviceIdName))
			{
				UE_CLOGF(!DeviceIdName.IsNone(), LogAudioEnumeration, Display, "FWindowsMMDeviceCache: Default Capture Role='%ls', Device='%ls'", ToString((EAudioDeviceRole)i), *GetFriendlyName(DeviceIdName));
				DefaultCaptureId[i] = DeviceIdName;
			}
		}
	}

	void FWindowsMMDeviceCache::OnDefaultCaptureDeviceChanged(const EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId)
	{
		FWriteScopeLock WriteLock(CacheMutationLock);
		check(InAudioDeviceRole < EAudioDeviceRole::COUNT);
		DefaultCaptureId[(int32)InAudioDeviceRole] = *DeviceId;
	}

	void FWindowsMMDeviceCache::OnDefaultRenderDeviceChanged(const EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId)
	{
		FWriteScopeLock WriteLock(CacheMutationLock);
		check(InAudioDeviceRole < EAudioDeviceRole::COUNT);
		DefaultRenderId[(int32)InAudioDeviceRole] = *DeviceId;
	}

	void FWindowsMMDeviceCache::OnDeviceAdded(const FString& DeviceId, bool bIsRender)
	{
		if (TOptional<FCacheEntry> NewDeviceEntry = BuildCacheEntry(DeviceId))
		{
			FWriteScopeLock WriteLock(CacheMutationLock);
			if (FCacheEntry* Entry = Cache.Find(NewDeviceEntry->DeviceId))
			{
				*Entry = MoveTemp(*NewDeviceEntry);
				UE_LOGF(LogAudioEnumeration, Warning, "FWindowsMMDeviceCache::OnDeviceAdded: Device already in cache. Replacing entry for DeviceID='%ls'", *DeviceId);
			}
			else
			{
				Cache.Emplace(NewDeviceEntry->DeviceId, MoveTemp(*NewDeviceEntry));
			}
		}
		else
		{
			UE_LOGF(LogAudioEnumeration, Warning, "FWindowsMMDeviceCache::OnDeviceAdded: Failed to add DeviceID='%ls' to cache. ", *DeviceId);
		}
	}

	void FWindowsMMDeviceCache::OnDeviceRemoved(const FString& DeviceId, bool)
	{
		FWriteScopeLock WriteLock(CacheMutationLock);
		FName DeviceIdName = *DeviceId;
		UE_CLOGF(!Cache.Contains(DeviceIdName), LogAudioEnumeration, Warning, "FWindowsMMDeviceCache::OnDeviceRemoved: DeviceId='%ls' was not in the cache. ", *DeviceId);
		Cache.Remove(DeviceIdName);
	}

	TOptional<FWindowsMMDeviceCache::FCacheEntry> FWindowsMMDeviceCache::BuildCacheEntry(const FString& DeviceId)
	{
		if (ensure(DeviceEnumerator))
		{
			TComPtr<IMMDevice> Device;
			if (SUCCEEDED(DeviceEnumerator->GetDevice(*DeviceId, &Device)))
			{
				FCacheEntry Info{ *DeviceId };
				if (EnumerateDeviceProps(Device, Info))
				{
					return Info;
				}
			}
		}
		return {};
	}

	FString FWindowsMMDeviceCache::GetFriendlyName(FName InDeviceId) const
	{
		if (const FCacheEntry* Entry = Cache.Find(InDeviceId))
		{
			return Entry->FriendlyName;
		}
		return TEXT("Unknown");
	}

	void FWindowsMMDeviceCache::OnDeviceStateChanged(const FString& DeviceId, const EAudioDeviceState InState, bool)
	{
		FName DeviceIdName = *DeviceId;
		
		// NOTE: If entry does not exist that's likely because a state change has preempted the OnDeviceAdded call.

		// Scope for Read-lock on Cache Map.
		FReadScopeLock ReadLock(CacheMutationLock);
		if (FCacheEntry* Entry = Cache.Find(DeviceIdName))
		{
			// Inner Write-Lock on Entry.
			FWriteScopeLock WriteLock(Entry->MutationLock);

			UE_CLOGF(WindowsMMCvarUtils::ShouldLogDeviceSwaps(), LogAudioEnumeration, Verbose, "FWindowsMMDeviceCache: DeviceName='%ls' - DeviceID='%ls' state changed from '%ls' to '%ls'.",
				*Entry->FriendlyName, *DeviceId, ToString(Entry->State), ToString(InState));

			Entry->State = InState;
		}
	}

	void FWindowsMMDeviceCache::InvalidateDevice(FName InDeviceId)
	{
		// Called when a consumer observes that a device reported Active is actually unreachable.
		// Cheap, surgical: flip one entry Active -> NotPresent under per-entry write lock so the
		// next swap-retry iteration sees it correctly and stops looping on the zombie ID.
		if (InDeviceId.IsNone())
		{
			return;
		}

		// Map read-lock is sufficient because we are not adding or removing entries; we are only
		// mutating an existing entry's State. The per-entry write lock serializes that mutation.
		FReadScopeLock ReadLock(CacheMutationLock);

		if (FCacheEntry* Entry = Cache.Find(InDeviceId))
		{
			FWriteScopeLock WriteLock(Entry->MutationLock);
			if (Entry->State == EAudioDeviceState::Active)
			{
				UE_LOGF(LogAudioEnumeration, Warning, "FWindowsMMDeviceCache::InvalidateDevice: FriendlyName='%ls' DeviceID='%ls' marked NotPresent; caller observed it unreachable.",
					*Entry->FriendlyName, *InDeviceId.ToString());
				Entry->State = EAudioDeviceState::NotPresent;
			}
		}
	}

	void FWindowsMMDeviceCache::OnFormatChanged(const FString& InDeviceId, const FFormatChangedData& InFormat)
	{
		FName DeviceName(InDeviceId);
		bool bNeedToEnumerateChannels = false;
		bool bDirty = false;

		FReadScopeLock MapReadLock(CacheMutationLock);
		if (FCacheEntry* Found = Cache.Find(DeviceName))
		{
			// Make a copy of the entry
			FCacheEntry EntryCopy(InDeviceId);
			{
				FReadScopeLock FoundReadLock(Found->MutationLock);
				EntryCopy = *Found;
			}

			if (EntryCopy.NumChannels != InFormat.NumChannels)
			{
				UE_CLOGF(WindowsMMCvarUtils::ShouldLogDeviceSwaps(), LogAudioEnumeration, Verbose, "FWindowsMMDeviceCache: DeviceID='%ls', Name='%ls' changed default format from %d channels to %d.", *InDeviceId, *EntryCopy.FriendlyName, EntryCopy.NumChannels, InFormat.NumChannels);
				EntryCopy.NumChannels = InFormat.NumChannels;
				bNeedToEnumerateChannels = true;
				bDirty = true;
			}
			if (EntryCopy.SampleRate != InFormat.SampleRate)
			{
				UE_CLOGF(WindowsMMCvarUtils::ShouldLogDeviceSwaps(), LogAudioEnumeration, Verbose, "FWindowsMMDeviceCache: DeviceID='%ls', Name='%ls' changed default format from %dhz to %dhz.", *InDeviceId, *EntryCopy.FriendlyName, EntryCopy.SampleRate, InFormat.SampleRate);
				EntryCopy.SampleRate = InFormat.SampleRate;
				bDirty = true;
			}
			if (EntryCopy.ChannelBitmask != InFormat.ChannelBitmask)
			{
				UE_CLOGF(WindowsMMCvarUtils::ShouldLogDeviceSwaps(), LogAudioEnumeration, Verbose, "FWindowsMMDeviceCache: DeviceID='%ls', Name='%ls' changed default format from 0x%x to 0x%x bitmask", *InDeviceId, *EntryCopy.FriendlyName, EntryCopy.ChannelBitmask, InFormat.ChannelBitmask);
				EntryCopy.ChannelBitmask = InFormat.ChannelBitmask;
				bNeedToEnumerateChannels = true;
				bDirty = true;
			}

			if (bNeedToEnumerateChannels)
			{
				UE_CLOGF(WindowsMMCvarUtils::ShouldLogDeviceSwaps(), LogAudioEnumeration, Verbose, "FWindowsMMDeviceCache: Channel Change, DeviceID='%ls', Name='%ls' OLD=[%ls]", *InDeviceId, *EntryCopy.FriendlyName, *ToFString(EntryCopy.OutputChannels));
				EnumerateChannelMask(InFormat.ChannelBitmask, EntryCopy);
				UE_CLOGF(WindowsMMCvarUtils::ShouldLogDeviceSwaps(), LogAudioEnumeration, Verbose, "FWindowsMMDeviceCache: Channel Change, DeviceID='%ls', Name='%ls' NEW=[%ls]", *InDeviceId, *EntryCopy.FriendlyName, *ToFString(EntryCopy.OutputChannels));
			}

			// Update the entire entry with one write.
			if (bDirty)
			{
				FWriteScopeLock FoundWriteLock(Found->MutationLock);
				*Found = EntryCopy;
			}
		}
	}

	void FWindowsMMDeviceCache::MakeDeviceInfo(const FCacheEntry& InEntry, FName InDefaultDevice, FAudioPlatformDeviceInfo& OutInfo) const
	{
		OutInfo.Reset();
		OutInfo.Name = InEntry.FriendlyName;
		OutInfo.DeviceId = InEntry.DeviceId.GetPlainNameString();
		OutInfo.NumChannels = InEntry.NumChannels;
		OutInfo.SampleRate = InEntry.SampleRate;
		OutInfo.OutputChannelArray = InEntry.OutputChannels;
		OutInfo.Format = EAudioMixerStreamDataFormat::Float;
		OutInfo.bIsSystemDefault = InEntry.DeviceId == InDefaultDevice;
	}

	TArray<FAudioPlatformDeviceInfo> FWindowsMMDeviceCache::GetAllActiveOutputDevices() const
	{
		SCOPED_NAMED_EVENT(FWindowsMMDeviceCache_GetAllActiveOutputDevices, FColor::Blue);

		// Find all active devices.
		TArray<FAudioPlatformDeviceInfo> ActiveDevices;

		if (IsAggregateDeviceSupportEnabled())
		{
			// Determine if there are any aggregate devices and place them at the beginning of the array
			ActiveDevices = SynthesizeAggregateDeviceList(EDeviceEndpointType::Render);
			Algo::Sort(ActiveDevices, [](const FAudioPlatformDeviceInfo& ItemA, const FAudioPlatformDeviceInfo& ItemB)
			{
				return ItemA.Name.Compare(ItemB.Name) < 0;
			});
		}

		TArray<FAudioPlatformDeviceInfo> NonAggregateDevices;

		// Read lock
		FReadScopeLock ReadLock(CacheMutationLock);
		NonAggregateDevices.Reserve(Cache.Num());

		// Ask for defaults once, as we are inside a read lock.
		FName DefaultRenderDeviceId = GetDefaultOutputDevice_NoLock();

		// Walk cache, read lock for each entry.
		for (const auto& i : Cache)
		{
			// Read lock on each entry.
			FReadScopeLock CacheEntryReadLock(i.Value.MutationLock);
			if (i.Value.State == EAudioDeviceState::Active &&
				i.Value.Type == EDeviceEndpointType::Render)
			{
				FAudioPlatformDeviceInfo& Info = NonAggregateDevices.Emplace_GetRef();
				MakeDeviceInfo(i.Value, DefaultRenderDeviceId, Info);
			}
		}

		Algo::Sort(NonAggregateDevices, [](const FAudioPlatformDeviceInfo& ItemA, const FAudioPlatformDeviceInfo& ItemB)
		{
			// See if both devices belong to an aggregate group. If so, then we use channel numbers
			// as the secondary sort criteria.
			FString ItemAHardwareName = ExtractAggregateDeviceName(ItemA.Name);
			FString ItemBHardwareName = ExtractAggregateDeviceName(ItemB.Name);
			
			int32 Result = ItemAHardwareName.Compare(ItemBHardwareName);
			if (Result == 0)
			{
				int32 ItemAChannelNumber = ExtractAggregateChannelNumber(ItemA.Name);
				int32 ItemBChannelNumber = ExtractAggregateChannelNumber(ItemB.Name);
				return ItemAChannelNumber < ItemBChannelNumber;
			}

			return Result < 0;
		});

		ActiveDevices.Append(NonAggregateDevices);
		
		// RVO
		return ActiveDevices;
	}

	bool FWindowsMMDeviceCache::IsAggregateHardwareDeviceId(const FName InDeviceID) const
	{
		FReadScopeLock ReadLock(CacheMutationLock);

		if (!InDeviceID.IsNone())
		{
			TArray<FCacheEntry> ActiveDevices;
			for (const auto& Entry : Cache)
			{
				// HardwareId is used for DeviceId with aggregate devices
				if (Entry.Value.HardwareId == InDeviceID)
				{
					return true;
				}
			}
		}

		return false;
	}

	TOptional<FAudioPlatformDeviceInfo> FWindowsMMDeviceCache::GetAggregateHardwareDeviceInfo(const FName InHardwareId, const EDeviceEndpointType InEndpointType) const
	{
		TSet<FCacheEntry, FCacheKeyFuncs> UniqueHardwareIds;
		TMap<FName, FDeviceChannelInfo> DeviceChannelInfos;

		GetHardwareInfo(UniqueHardwareIds, DeviceChannelInfos, InEndpointType);

		for (const auto& Entry : UniqueHardwareIds)
		{
			if (Entry.HardwareId == InHardwareId)
			{
				if (ensure(DeviceChannelInfos.Contains(Entry.HardwareId)))
				{
					return CreateAggregateDeviceInfo(Entry, DeviceChannelInfos[Entry.HardwareId]);
				}
			}
		}

		return {};
	}
	
	TArray<FAudioPlatformDeviceInfo> FWindowsMMDeviceCache::GetLogicalAggregateDevices(const FName InHardwareId, const EDeviceEndpointType InEndpointType) const
	{
		TArray<FAudioPlatformDeviceInfo> AggregateDevice;
		FReadScopeLock ReadLock(CacheMutationLock);

		TArray<FCacheEntry> ActiveDevices;
		for (const auto& Entry : Cache)
		{
			if (Entry.Value.HardwareId == InHardwareId &&
				Entry.Value.State == EAudioDeviceState::Active &&
				Entry.Value.Type == InEndpointType)
			{
				ActiveDevices.Emplace(Entry.Value);
			}
		}

		// Sort the devices according to their filter id which usually ends with a vendor specific
		// string which is sortable
		Algo::Sort(ActiveDevices, [](const FCacheEntry& InEntryA, const FCacheEntry& InEntryB)
		{
			return InEntryA.FilterId.Compare(InEntryB.FilterId) < 0;
		});

		for (const FCacheEntry& Device : ActiveDevices)
		{
			FAudioPlatformDeviceInfo& Info = AggregateDevice.Emplace_GetRef();
			MakeDeviceInfo(Device, GetDefaultOutputDevice_NoLock(), Info);
		}

		return AggregateDevice;
	}
	
	void FWindowsMMDeviceCache::GetHardwareInfo(TSet<FCacheEntry, FCacheKeyFuncs>& OutUniqueHardwareIds,
												TMap<FName, FDeviceChannelInfo>& OutDeviceChannelInfos,
												EDeviceEndpointType InType) const
	{
		FReadScopeLock ReadLock(CacheMutationLock);
		for (const auto& Entry : Cache)
		{
			if (Entry.Value.State == EAudioDeviceState::Active && Entry.Value.Type == InType)
			{
				OutUniqueHardwareIds.Add(Entry.Value);

				// Accumulate channel counts for the aggregate devices
				if (OutDeviceChannelInfos.Contains(Entry.Value.HardwareId))
				{
					FDeviceChannelInfo* ChannelInfo = OutDeviceChannelInfos.Find(Entry.Value.HardwareId);
					ensure(ChannelInfo->LogicDeviceChannelCount == Entry.Value.NumChannels);

					ChannelInfo->TotalChannelCount += Entry.Value.NumChannels;
				}
				else
				{
					FDeviceChannelInfo& ChannelInfo = OutDeviceChannelInfos.Add(Entry.Value.HardwareId);
					ChannelInfo.LogicDeviceChannelCount = Entry.Value.NumChannels;
					ChannelInfo.TotalChannelCount += Entry.Value.NumChannels;
				}
			}
		}
	}

	FAudioPlatformDeviceInfo FWindowsMMDeviceCache::CreateAggregateDeviceInfo(const FCacheEntry& InCacheEntry, const FDeviceChannelInfo& InDeviceChannelInfo)
	{
		FAudioPlatformDeviceInfo Info;
		
		// Every Windows audio device contains the hardware name at the end of the string in parentheses
		Info.Name = ExtractAggregateDeviceName(InCacheEntry.FriendlyName);
		// Synthesize a device id from the hardware id which is unique
		Info.DeviceId = InCacheEntry.HardwareId.GetPlainNameString();
		Info.NumChannels = InCacheEntry.NumChannels;
		Info.SampleRate = InCacheEntry.SampleRate;
		Info.OutputChannelArray = InCacheEntry.OutputChannels;
		Info.Format = EAudioMixerStreamDataFormat::Float;
		Info.bIsSystemDefault = false;

		int32 NumDirectOuts = InDeviceChannelInfo.TotalChannelCount - InDeviceChannelInfo.LogicDeviceChannelCount;
		Info.NumDirectOutChannels = FMath::Max(0, NumDirectOuts);

		return Info;
	}
	
	TArray<FAudioPlatformDeviceInfo> FWindowsMMDeviceCache::SynthesizeAggregateDeviceList(EDeviceEndpointType InType) const
	{
		TSet<FCacheEntry, FCacheKeyFuncs> UniqueHardwareIds;
		TMap<FName, FDeviceChannelInfo> DeviceChannelInfos;

		GetHardwareInfo(UniqueHardwareIds, DeviceChannelInfos, InType);
		TArray<FAudioPlatformDeviceInfo> AggregateDeviceList;

		for (const auto& Entry : UniqueHardwareIds)
		{
			if (ensure(DeviceChannelInfos.Contains(Entry.HardwareId)))
			{
				FAudioPlatformDeviceInfo Info = CreateAggregateDeviceInfo(Entry, DeviceChannelInfos[Entry.HardwareId]);

				// Only consider aggregate devices that have direct out channels
				if (Info.NumDirectOutChannels > 0)
				{
					AggregateDeviceList.Emplace(MoveTemp(Info));
				}
			}
		}

		return AggregateDeviceList;
	}

	FString FWindowsMMDeviceCache::ExtractAggregateDeviceName(const FString& InName)
	{
		// The hardware name is in parentheses at the end of the string
		const FRegexPattern RegexPattern(TEXT("\\(([^\\(\\)]+)\\)$"));
		FRegexMatcher RegexMatcher(RegexPattern, InName);

		if (RegexMatcher.FindNext())
		{
			return RegexMatcher.GetCaptureGroup(1);
		}
		
		return InName;
	}

	int32 FWindowsMMDeviceCache::ExtractAggregateChannelNumber(const FString& InName)
	{
		// The hardware name is in parentheses at the end of the string
		const FRegexPattern RegexPattern(TEXT("^.+?\\(([\\d]+)-[\\d]+\\)"));
		FRegexMatcher RegexMatcher(RegexPattern, InName);

		if (RegexMatcher.FindNext())
		{
			FString ChannelNumber = RegexMatcher.GetCaptureGroup(1);
			return FCString::Atoi(*ChannelNumber);
		}
		
		return 0;
	}

	FName FWindowsMMDeviceCache::GetDefaultOutputDevice(EAudioDeviceRole InRole) const
	{
		FReadScopeLock MapReadLock(CacheMutationLock);

		if (ensure(InRole >= EAudioDeviceRole::Console && InRole < EAudioDeviceRole::COUNT))
		{
			return DefaultRenderId[(int32)InRole];
		}
		return NAME_None;
	}

	FName FWindowsMMDeviceCache::GetDefaultOutputDevice_NoLock() const
	{
		if (!DefaultRenderId[(int32)EAudioDeviceRole::Console].IsNone())
		{
			return DefaultRenderId[(int32)EAudioDeviceRole::Console];
		}
		if (!DefaultRenderId[(int32)EAudioDeviceRole::Multimedia].IsNone())
		{
			return DefaultRenderId[(int32)EAudioDeviceRole::Multimedia];
		}
		return NAME_None;
	}

	TOptional<FAudioPlatformDeviceInfo> FWindowsMMDeviceCache::FindDefaultOutputDevice() const
	{
		return FindActiveOutputDevice(NAME_None);
	}

	TOptional<FAudioPlatformDeviceInfo> FWindowsMMDeviceCache::FindActiveOutputDevice(FName InDeviceID) const
	{
		SCOPED_NAMED_EVENT(FWindowsMMDeviceCache_FindActiveOutputDevice, FColor::Blue);

		FReadScopeLock MapReadLock(CacheMutationLock);

		// Ask for default here as we are inside the read lock.
		const FName DefaultOutputDevice = GetDefaultOutputDevice_NoLock();

		// Asking for Default?
		if (InDeviceID.IsNone())
		{
			InDeviceID = DefaultOutputDevice;
			if (InDeviceID.IsNone())
			{
				// No default set, fail.
				return {};
			}
		}

		// Find entry matching that device ID.
		if (const FCacheEntry* Found = Cache.Find(InDeviceID))
		{
			FReadScopeLock EntryReadLock(Found->MutationLock);
			if (Found->State == EAudioDeviceState::Active &&
				Found->Type == EDeviceEndpointType::Render)
			{
				FAudioPlatformDeviceInfo Info;
				MakeDeviceInfo(*Found, DefaultOutputDevice, Info);
				return Info;
			}
		}

		if (bIsAggregateDeviceSupportEnabled)
		{
			/** Returns the device info for an aggregate audio device. Note that this is a virtual device in that it is not
			 *  a device returned by the OS device enumerator. It is a device synthesized for the purpose of identifying
			 *  an aggregate device as a single, unified device. This device cannot be instantiated as-is. It's device Id
			 *  can be used with GetAggregateHardwareDeviceInfo() to get an array of the logical devices that make up the 
			 *  aggregate and can be instantiated as a group to form an aggregate device at runtime.
			 */
			TOptional<FAudioPlatformDeviceInfo> Info = GetAggregateHardwareDeviceInfo(InDeviceID, EDeviceEndpointType::Render);
			if (Info.IsSet())
			{
				return Info;
			}
		}
		
		// Fail.
		return {};
	}

	bool FWindowsMMDeviceCache::IsAggregateDeviceSupportEnabled() const
	{
		return bIsAggregateDeviceSupportEnabled || WindowsMMCvarUtils::IsAggregateDeviceSupportCVarEnabled();
	}
} // namespace Audio
