// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsMMNotificationClient.h"

#include "Windows/AllowWindowsPlatformTypes.h"

THIRD_PARTY_INCLUDES_START
#include <Functiondiscoverykeys_devpkey.h>
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformTypes.h"

#include "WindowsMMCvarUtils.h"
#include "WindowsMMStringUtils.h"
#include "ConversionHelpers.h"
#include "WindowsMMDeviceEnumerationLog.h"

namespace Audio
{
#if PLATFORM_WINDOWS
	bool FWindowsMMNotificationClient::RegisterForSessionNotifications(const TComPtr<IMMDevice>& InDevice)
	{
		FScopeLock Lock(&SessionRegistrationCS);

		// If we're already listening to this device, we can early out.
		if (DeviceListeningToSessionEvents == InDevice)
		{
			return true;
		}

		UnregisterForSessionNotifications();

		DeviceListeningToSessionEvents = InDevice;

		if (InDevice)
		{
			if (SUCCEEDED(InDevice->Activate(__uuidof(IAudioSessionManager), CLSCTX_INPROC_SERVER, NULL, (void**)&SessionManager)))
			{
				if (SUCCEEDED(SessionManager->GetAudioSessionControl(NULL, 0, &SessionControls)))
				{
					if (SUCCEEDED(SessionControls->RegisterAudioSessionNotification(this)))
					{
						UE_LOGF(LogAudioEnumeration, Verbose, "FWindowsMMNotificationClient: Registering for sessions events for '%ls'", *GetFriendlyName(DeviceListeningToSessionEvents.Get()));
						return true;
					}
				}
			}
		}
		return false;
	}

	bool FWindowsMMNotificationClient::RegisterForSessionNotifications(const FString& InDeviceId)
	{
		if (TComPtr<IMMDevice> Device = GetDevice(InDeviceId))
		{
			return RegisterForSessionNotifications(Device);
		}
		return false;
	}


	HRESULT STDMETHODCALLTYPE FWindowsMMNotificationClient::OnSessionDisconnected(AudioSessionDisconnectReason InDisconnectReason)
	{
		UE_CLOGF(WindowsMMCvarUtils::ShouldLogDeviceSwaps(), LogAudioEnumeration, Verbose, "Session Disconnect: Reason=%ls, DeviceBound=%ls, HasDisconnectSessionHappened=%d",
			ToString(InDisconnectReason), *GetFriendlyName(DeviceListeningToSessionEvents), (int32)bHasDisconnectSessionHappened);

		if (!bHasDisconnectSessionHappened)
		{
			{
				FReadScopeLock Lock(ListenersSetRwLock);
				Audio::IAudioMixerDeviceChangedListener::EDisconnectReason Reason = AudioSessionDisconnectToEDisconnectReason(InDisconnectReason);
				for (Audio::IAudioMixerDeviceChangedListener* i : Listeners)
				{
					i->OnSessionDisconnect(Reason);
				}
			}

			// Mark this true.
			bHasDisconnectSessionHappened = true;
		}

		return S_OK;
	}

	HRESULT STDMETHODCALLTYPE FWindowsMMNotificationClient::OnStateChanged(AudioSessionState NewState)
	{
		return S_OK;
	}

	HRESULT STDMETHODCALLTYPE FWindowsMMNotificationClient::OnGroupingParamChanged(LPCGUID NewGroupingParam, LPCGUID EventContext)
	{
		return S_OK;
	}

	HRESULT STDMETHODCALLTYPE FWindowsMMNotificationClient::OnChannelVolumeChanged(DWORD ChannelCount, float NewChannelVolumeArray[], DWORD ChangedChannel, LPCGUID EventContext)
	{
		return S_OK;
	}

	HRESULT STDMETHODCALLTYPE FWindowsMMNotificationClient::OnSimpleVolumeChanged(float NewVolume, BOOL NewMute, LPCGUID EventContext)
	{
		return S_OK;
	}

	HRESULT STDMETHODCALLTYPE FWindowsMMNotificationClient::OnIconPathChanged(LPCWSTR NewIconPath, LPCGUID EventContext)
	{
		return S_OK;
	}

	HRESULT STDMETHODCALLTYPE FWindowsMMNotificationClient::OnDisplayNameChanged(LPCWSTR NewDisplayName, LPCGUID EventContext)
	{
		return S_OK;
	}
#endif //PLATFORM_WINDOWS

	void FWindowsMMNotificationClient::UnRegisterDeviceDeviceChangedListener(Audio::IAudioMixerDeviceChangedListener* DeviceChangedListener)
	{
		// Modifying container so get full write lock
		FWriteScopeLock Lock(ListenersSetRwLock);
		Listeners.Remove(DeviceChangedListener);
	}

	void FWindowsMMNotificationClient::RegisterDeviceChangedListener(Audio::IAudioMixerDeviceChangedListener* DeviceChangedListener)
	{
		// Modifying container so get full write lock
		FWriteScopeLock Lock(ListenersSetRwLock);
		Listeners.Add(DeviceChangedListener);
	}

}

#include "Windows/AllowWindowsPlatformAtomics.h"

namespace Audio
{
	ULONG STDMETHODCALLTYPE FWindowsMMNotificationClient::AddRef()
	{
		return InterlockedIncrement(&Ref);
	}

	ULONG STDMETHODCALLTYPE FWindowsMMNotificationClient::Release()
	{
		ULONG ulRef = InterlockedDecrement(&Ref);
		if (0 == ulRef)
		{
			delete this;
		}
		return ulRef;
	}
}

#include "Windows/HideWindowsPlatformAtomics.h"

namespace Audio
{
	HRESULT STDMETHODCALLTYPE FWindowsMMNotificationClient::QueryInterface(const IID& IId, void** UnknownPtrPtr)
	{
		// Three rules of QueryInterface: https://docs.microsoft.com/en-us/windows/win32/com/rules-for-implementing-queryinterface
		// 1. Objects must have identity.
		// 2. The set of interfaces on an object instance must be static.
		// 3. It must be possible to query successfully for any interface on an object from any other interface.

		// If ppvObject(the address) is nullptr, then this method returns E_POINTER.
		if (!UnknownPtrPtr)
		{
			return E_POINTER;
		}

		// https://docs.microsoft.com/en-us/windows/win32/com/implementing-reference-counting
		// Whenever a client calls a method(or API function), such as QueryInterface, that returns a new interface pointer, 
		// the method being called is responsible for incrementing the reference count through the returned pointer.
		// For example, when a client first creates an object, it receives an interface pointer to an object that, 
		// from the client's point of view, has a reference count of one. If the client then calls AddRef on the interface pointer, 
		// the reference count becomes two. The client must call Release twice on the interface pointer to drop all of its references to the object.
		if (IId == __uuidof(IMMNotificationClient) || IId == __uuidof(IUnknown))
		{
			*UnknownPtrPtr = (IMMNotificationClient*)(this);
			AddRef();
			return S_OK;
		}
		else if (IId == __uuidof(IAudioSessionEvents))
		{
			*UnknownPtrPtr = (IAudioSessionEvents*)this;
			AddRef();
			return S_OK;
		}


		// This method returns S_OK if the interface is supported, and E_NOINTERFACE otherwise.
		*UnknownPtrPtr = nullptr;
		return E_NOINTERFACE;
	}

	HRESULT STDMETHODCALLTYPE FWindowsMMNotificationClient::OnPropertyValueChanged(LPCWSTR pwstrDeviceId, const PROPERTYKEY key)
	{
		UE_CLOGF(WindowsMMCvarUtils::ShouldLogDeviceSwaps(), LogAudioEnumeration, Verbose, "OnPropertyValueChanged: %ls : %ls", *GetFriendlyName(pwstrDeviceId), *ToFString(key));

		if (key.fmtid == PKEY_AudioEngine_DeviceFormat.fmtid)
		{
			// Get device.
			FString DeviceId = pwstrDeviceId;
			TComPtr<IMMDevice> Device;
			HRESULT Hr = DeviceEnumerator->GetDevice(*DeviceId, &Device);

			// Get property store.
			TComPtr<IPropertyStore> PropertyStore;
			if (SUCCEEDED(Hr) && Device)
			{
				Hr = Device->OpenPropertyStore(STGM_READ, &PropertyStore);
				if (SUCCEEDED(Hr) && PropertyStore)
				{
					// Device Format
					PROPVARIANT Prop;
					PropVariantInit(&Prop);

					if (key.fmtid == PKEY_AudioEngine_DeviceFormat.fmtid)
					{
						// WAVEFORMATEX blobs.
						if (SUCCEEDED(PropertyStore->GetValue(key, &Prop)) && Prop.blob.pBlobData)
						{
							const WAVEFORMATEX* WaveFormatEx = (const WAVEFORMATEX*)(Prop.blob.pBlobData);

							Audio::IAudioMixerDeviceChangedListener::FFormatChangedData FormatChanged;
							FormatChanged.NumChannels = FMath::Clamp((int32)WaveFormatEx->nChannels, 2, 8);
							FormatChanged.SampleRate = WaveFormatEx->nSamplesPerSec;
							FormatChanged.ChannelBitmask = WaveFormatEx->wFormatTag == WAVE_FORMAT_EXTENSIBLE ?
								((const WAVEFORMATEXTENSIBLE*)WaveFormatEx)->dwChannelMask : 0;

							FReadScopeLock ReadLock(ListenersSetRwLock);
							for (Audio::IAudioMixerDeviceChangedListener* i : Listeners)
							{
								i->OnFormatChanged(DeviceId, FormatChanged);
							}
						}
					}
					PropVariantClear(&Prop);
				}
			}
		}
		return S_OK;
	}

	TComPtr<IMMDevice> FWindowsMMNotificationClient::GetDevice(const FString InDeviceID) const
	{
		// Get device.
		TComPtr<IMMDevice> Device;
		HRESULT Hr = DeviceEnumerator->GetDevice(*InDeviceID, &Device);
		if (SUCCEEDED(Hr))
		{
			return Device;
		}

		// Fail.
		return {};
	}

	uint32 FWindowsMMNotificationClient::ReleaseClient()
	{
		return Release();
	}

	FString FWindowsMMNotificationClient::GetFriendlyName(const TComPtr<IMMDevice>& InDevice)
	{
		FString FriendlyName = "[No Friendly Name for Device]";

		if (InDevice)
		{
			// Get property store.
			TComPtr<IPropertyStore> PropStore;
			HRESULT Hr = InDevice->OpenPropertyStore(STGM_READ, &PropStore);

			// Get friendly name.
			if (SUCCEEDED(Hr) && PropStore)
			{
				PROPVARIANT PropString;
				PropVariantInit(&PropString);

				// Get the endpoint device's friendly-name property.
				Hr = PropStore->GetValue(PKEY_Device_FriendlyName, &PropString);
				if (SUCCEEDED(Hr))
				{
					// Copy friendly name.
					if (PropString.pwszVal)
					{
						FriendlyName = PropString.pwszVal;
					}
				}

				PropVariantClear(&PropString);
			}
		}
		return FriendlyName;
	}

	FString FWindowsMMNotificationClient::GetFriendlyName(const FString InDeviceID)
	{
		if (InDeviceID.IsEmpty())
		{
			return TEXT("System Default");
		}

		FString FriendlyName = "[No Friendly Name for Device]";

		// Get device.
		if (TComPtr<IMMDevice> Device = GetDevice(*InDeviceID))
		{
			return GetFriendlyName(Device);
		}
		return FriendlyName;
	}


	HRESULT STDMETHODCALLTYPE FWindowsMMNotificationClient::OnDeviceStateChanged(LPCWSTR pwstrDeviceId, DWORD dwNewState)
	{
		UE_CLOGF(WindowsMMCvarUtils::ShouldLogDeviceSwaps(), LogAudioEnumeration, Display, "FWindowsMMNotificationClient: OnDeviceStateChanged: %ls, %d", *GetFriendlyName(pwstrDeviceId), dwNewState);

		if (WindowsMMCvarUtils::ShouldIgnoreDeviceSwaps())
		{
			return S_OK;
		}

		bool bIsRender = IsRenderDevice(pwstrDeviceId);
		if (dwNewState == DEVICE_STATE_ACTIVE || dwNewState == DEVICE_STATE_DISABLED || dwNewState == DEVICE_STATE_UNPLUGGED || dwNewState == DEVICE_STATE_NOTPRESENT)
		{
			Audio::EAudioDeviceState State = ConvertWordToDeviceState(dwNewState);

			FString DeviceString(pwstrDeviceId);
			FReadScopeLock ReadLock(ListenersSetRwLock);
			for (Audio::IAudioMixerDeviceChangedListener* Listener : Listeners)
			{
				Listener->OnDeviceStateChanged(DeviceString, State, bIsRender);
			}
		}
		return S_OK;
	}

	HRESULT STDMETHODCALLTYPE FWindowsMMNotificationClient::OnDeviceRemoved(LPCWSTR pwstrDeviceId)
	{
		UE_CLOGF(WindowsMMCvarUtils::ShouldLogDeviceSwaps(), LogAudioEnumeration, Display, "FWindowsMMNotificationClient: OnDeviceRemoved: %ls", *GetFriendlyName(pwstrDeviceId));

		if (WindowsMMCvarUtils::ShouldIgnoreDeviceSwaps())
		{
			return S_OK;
		}

		bool bIsRender = IsRenderDevice(pwstrDeviceId);
		FString DeviceString(pwstrDeviceId);
		FReadScopeLock ReadLock(ListenersSetRwLock);
		for (Audio::IAudioMixerDeviceChangedListener* Listener : Listeners)
		{
			Listener->OnDeviceRemoved(DeviceString, bIsRender);
		}
		return S_OK;
	}

	HRESULT STDMETHODCALLTYPE FWindowsMMNotificationClient::OnDeviceAdded(LPCWSTR pwstrDeviceId)
	{
		UE_CLOGF(WindowsMMCvarUtils::ShouldLogDeviceSwaps(), LogAudioEnumeration, Display, "FWindowsMMNotificationClient: OnDeviceAdded: %ls", *GetFriendlyName(pwstrDeviceId));

		if (WindowsMMCvarUtils::ShouldIgnoreDeviceSwaps())
		{
			return S_OK;
		}

		bool bIsRender = IsRenderDevice(pwstrDeviceId);
		FString DeviceString(pwstrDeviceId);
		FReadScopeLock ReadLock(ListenersSetRwLock);
		for (Audio::IAudioMixerDeviceChangedListener* Listener : Listeners)
		{
			Listener->OnDeviceAdded(DeviceString, bIsRender);
		}
		return S_OK;
	}

	bool FWindowsMMNotificationClient::IsRenderDevice(const FString& InDeviceId) const
	{
		bool bIsRender = true;
		if (TComPtr<IMMDevice> Device = GetDevice(InDeviceId))
		{
			TComPtr<IMMEndpoint> Endpoint;
			if (SUCCEEDED(Device->QueryInterface(IID_PPV_ARGS(&Endpoint))))
			{
				EDataFlow DataFlow = eRender;
				if (SUCCEEDED(Endpoint->GetDataFlow(&DataFlow)))
				{
					bIsRender = DataFlow == eRender;
				}
			}
		}
		return bIsRender;
	}

	HRESULT STDMETHODCALLTYPE FWindowsMMNotificationClient::OnDefaultDeviceChanged(EDataFlow InFlow, ERole InRole, LPCWSTR pwstrDeviceId)
	{
		UE_CLOGF(WindowsMMCvarUtils::ShouldLogDeviceSwaps(), LogAudioEnumeration, Display,
			"FWindowsMMNotificationClient: OnDefaultDeviceChanged: %ls, %ls, %ls - %ls", ToString(InFlow), ToString(InRole), pwstrDeviceId, *GetFriendlyName(pwstrDeviceId));

		Audio::EAudioDeviceRole AudioDeviceRole;

		if (WindowsMMCvarUtils::ShouldIgnoreDeviceSwaps())
		{
			return S_OK;
		}

		if (InRole == eConsole)
		{
			AudioDeviceRole = Audio::EAudioDeviceRole::Console;
		}
		else if (InRole == eMultimedia)
		{
			AudioDeviceRole = Audio::EAudioDeviceRole::Multimedia;
		}
		else
		{
			AudioDeviceRole = Audio::EAudioDeviceRole::Communications;
		}

		FString DeviceString(pwstrDeviceId);
		if (InFlow == eRender)
		{
			FReadScopeLock Lock(ListenersSetRwLock);
			for (Audio::IAudioMixerDeviceChangedListener* Listener : Listeners)
			{
				Listener->OnDefaultRenderDeviceChanged(AudioDeviceRole, DeviceString);
			}
		}
		else if (InFlow == eCapture)
		{
			FReadScopeLock Lock(ListenersSetRwLock);
			for (Audio::IAudioMixerDeviceChangedListener* Listener : Listeners)
			{
				Listener->OnDefaultCaptureDeviceChanged(AudioDeviceRole, DeviceString);
			}
		}
		else
		{
			FReadScopeLock Lock(ListenersSetRwLock);
			for (Audio::IAudioMixerDeviceChangedListener* Listener : Listeners)
			{
				Listener->OnDefaultCaptureDeviceChanged(AudioDeviceRole, DeviceString);
				Listener->OnDefaultRenderDeviceChanged(AudioDeviceRole, DeviceString);
			}
		}
		return S_OK;
	}

	FWindowsMMNotificationClient::~FWindowsMMNotificationClient()
	{
#if PLATFORM_WINDOWS
		UnregisterForSessionNotifications();
#endif //PLATFORM_WINDOWS

		if (DeviceEnumerator)
		{
			DeviceEnumerator->UnregisterEndpointNotificationCallback(this);
		}

#if PLATFORM_WINDOWS
		if (bComInitialized)
		{
			FWindowsPlatformMisc::CoUninitialize();
		}
#endif //PLATFORM_WINDOWS
	}

#if PLATFORM_WINDOWS
	void FWindowsMMNotificationClient::UnregisterForSessionNotifications()
	{
		FScopeLock Lock(&SessionRegistrationCS);

		// Unregister for any device we're already listening to.
		if (SessionControls)
		{
			UE_LOGF(LogAudioEnumeration, Verbose, "FWindowsMMNotificationClient: Unregistering for sessions events for device '%ls'", DeviceListeningToSessionEvents ? *GetFriendlyName(DeviceListeningToSessionEvents.Get()) : TEXT("None"));
			SessionControls->UnregisterAudioSessionNotification(this);
			SessionControls.Reset();
		}
		if (SessionManager)
		{
			SessionManager.Reset();
		}

		DeviceListeningToSessionEvents.Reset();

		// Reset this flag.
		bHasDisconnectSessionHappened = false;
	}
#endif //PLATFORM_WINDOWS

	FWindowsMMNotificationClient::FWindowsMMNotificationClient() 
	{
#if PLATFORM_WINDOWS
		bComInitialized = FWindowsPlatformMisc::CoInitialize();
#endif //PLATFORM_WINDOWS

		HRESULT Result = CoCreateInstance(__uuidof(MMDeviceEnumerator), nullptr, CLSCTX_INPROC_SERVER, IID_PPV_ARGS(&DeviceEnumerator));
		if (Result == S_OK)
		{
			DeviceEnumerator->RegisterEndpointNotificationCallback(this);
		}

#if PLATFORM_WINDOWS
		// Register for session events from default endpoint.
		if (DeviceEnumerator)
		{
			TComPtr<IMMDevice> DefaultDevice;
			if (SUCCEEDED(DeviceEnumerator->GetDefaultAudioEndpoint(eRender, eConsole, &DefaultDevice)))
			{
				RegisterForSessionNotifications(DefaultDevice);
			}
		}
#endif //PLATFORM_WINDOWS
	}

}// namespace Audio
