// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsMMStringUtils.h"

#include "Containers/UnrealString.h"
#include "Misc/AssertionMacros.h"

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <winerror.h>
#include <audiopolicy.h>			// IAudioSessionEvents
#if !NO_LOGGING
// PSStringFromPropertyKey needs this lib, which we use for logging the Property GUIDs.
#pragma comment(lib, "Propsys.lib")
#endif //!NO_LOGGING
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

namespace Audio
{
	const TCHAR* ToString(AudioSessionDisconnectReason InDisconnectReason)
	{
#if !NO_LOGGING
#define CASE_TO_STRING(X) case AudioSessionDisconnectReason::X: return TEXT(#X)
		switch (InDisconnectReason)
		{
			CASE_TO_STRING(DisconnectReasonDeviceRemoval);
			CASE_TO_STRING(DisconnectReasonServerShutdown);
			CASE_TO_STRING(DisconnectReasonFormatChanged);
			CASE_TO_STRING(DisconnectReasonSessionLogoff);
			CASE_TO_STRING(DisconnectReasonSessionDisconnected);
			CASE_TO_STRING(DisconnectReasonExclusiveModeOverride);
		default:
			checkNoEntry();
			break;
		}
#undef CASE_TO_STRING
#endif //!NO_LOGGING
		return TEXT("Unknown");
	}

	const TCHAR* ToString(ERole InRole)
	{
#if !NO_LOGGING
#define CASE_TO_STRING(X) case ERole::X: return TEXT(#X)
		switch (InRole)
		{
			CASE_TO_STRING(eConsole);
			CASE_TO_STRING(eMultimedia);
			CASE_TO_STRING(eCommunications);
		default:
			checkNoEntry();
			break;
		}
#undef CASE_TO_STRING
#endif //!NO_LOGGING
		return TEXT("Unknown");
	}

	const TCHAR* ToString(EDataFlow InFlow)
	{
#if !NO_LOGGING
#define CASE_TO_STRING(X) case EDataFlow::X: return TEXT(#X)
		switch (InFlow)
		{
			CASE_TO_STRING(eRender);
			CASE_TO_STRING(eCapture);
			CASE_TO_STRING(eAll);
		default:
			checkNoEntry();
			break;
		}
#endif //!NO_LOGGING
#undef CASE_TO_STRING
		return TEXT("Unknown");
	}

	FString ToFString(const PROPERTYKEY InKey)
	{
#if PLATFORM_WINDOWS
#if !NO_LOGGING
#define IF_PROP_STRING(PROP) if(InKey.fmtid == PROP.fmtid) return TEXT(#PROP)

		IF_PROP_STRING(PKEY_AudioEndpoint_PhysicalSpeakers);
		IF_PROP_STRING(PKEY_AudioEngine_DeviceFormat);
		IF_PROP_STRING(PKEY_AudioEngine_OEMFormat);
		IF_PROP_STRING(PKEY_AudioEndpoint_Association);
		IF_PROP_STRING(PKEY_AudioEndpoint_ControlPanelPageProvider);
		IF_PROP_STRING(PKEY_AudioEndpoint_Disable_SysFx);
		IF_PROP_STRING(PKEY_AudioEndpoint_FormFactor);
		IF_PROP_STRING(PKEY_AudioEndpoint_FullRangeSpeakers);
		IF_PROP_STRING(PKEY_AudioEndpoint_GUID);
		IF_PROP_STRING(PKEY_AudioEndpoint_Supports_EventDriven_Mode);

#undef IF_PROP_STRING

		TCHAR KeyString[PKEYSTR_MAX];
		HRESULT HR = PSStringFromPropertyKey(InKey, KeyString, ARRAYSIZE(KeyString));
		if (SUCCEEDED(HR))
		{
			return FString(KeyString);
		}
#endif //!NO_LOGGING
#endif //PLATFORM_WINDOWS
		return TEXT("Unknown");
	}

	const TCHAR* ToString(EAudioDeviceRole InRole)
	{
#define CASE_TO_STRING(X) case EAudioDeviceRole::X: return TEXT(#X)
		switch (InRole)
		{
			CASE_TO_STRING(Console);
			CASE_TO_STRING(Multimedia);
			CASE_TO_STRING(Communications);
		default:
			return TEXT("Unknown");
		}
#undef CASE_TO_STRING
	}

	const TCHAR* ToString(EAudioDeviceState InState)
	{
#define CASE_TO_STRING(X) case EAudioDeviceState::X: return TEXT(#X)
		switch (InState)
		{
			CASE_TO_STRING(Active);
			CASE_TO_STRING(Disabled);
			CASE_TO_STRING(NotPresent);
			CASE_TO_STRING(Unplugged);
		default:
			return TEXT("Unknown");
		}
#undef CASE_TO_STRING
	}

	FString ToFString(const TArray<EAudioMixerChannel::Type>& InChannels)
	{
		FString ChannelString;
		static const int32 ApproxChannelNameLength = 18;
		ChannelString.Reserve(ApproxChannelNameLength * InChannels.Num());
		for (EAudioMixerChannel::Type i : InChannels)
		{
			ChannelString.Append(EAudioMixerChannel::ToString(i));
			ChannelString.Append(TEXT("|"));
		}
		return ChannelString;
	}

	FString AudioClientErrorToFString(HRESULT InResult)
	{
#define CASE_AND_STRING(RESULT) case HRESULT(RESULT): return TEXT(#RESULT)

		switch (InResult)
		{
		case HRESULT(REGDB_E_CLASSNOTREG):				return TEXT("REGDB_E_CLASSNOTREG");
		case HRESULT(CLASS_E_NOAGGREGATION):			return TEXT("CLASS_E_NOAGGREGATION");
		case HRESULT(E_NOINTERFACE):					return TEXT("E_NOINTERFACE");
		case HRESULT(E_POINTER):						return TEXT("E_POINTER");
		case HRESULT(E_INVALIDARG):						return TEXT("E_INVALIDARG");
		case HRESULT(E_OUTOFMEMORY):					return TEXT("E_OUTOFMEMORY");

		// AudioClient.h
		CASE_AND_STRING(AUDCLNT_E_NOT_INITIALIZED);
		CASE_AND_STRING(AUDCLNT_E_ALREADY_INITIALIZED);
		CASE_AND_STRING(AUDCLNT_E_WRONG_ENDPOINT_TYPE);
		CASE_AND_STRING(AUDCLNT_E_DEVICE_INVALIDATED);
		CASE_AND_STRING(AUDCLNT_E_NOT_STOPPED);
		CASE_AND_STRING(AUDCLNT_E_BUFFER_TOO_LARGE);
		CASE_AND_STRING(AUDCLNT_E_OUT_OF_ORDER);
		CASE_AND_STRING(AUDCLNT_E_UNSUPPORTED_FORMAT);
		CASE_AND_STRING(AUDCLNT_E_INVALID_SIZE);
		CASE_AND_STRING(AUDCLNT_E_DEVICE_IN_USE);
		CASE_AND_STRING(AUDCLNT_E_BUFFER_OPERATION_PENDING);
		CASE_AND_STRING(AUDCLNT_E_THREAD_NOT_REGISTERED);
		CASE_AND_STRING(AUDCLNT_E_EXCLUSIVE_MODE_NOT_ALLOWED);
		CASE_AND_STRING(AUDCLNT_E_ENDPOINT_CREATE_FAILED);
		CASE_AND_STRING(AUDCLNT_E_SERVICE_NOT_RUNNING);
		CASE_AND_STRING(AUDCLNT_E_EVENTHANDLE_NOT_EXPECTED);
		CASE_AND_STRING(AUDCLNT_E_EXCLUSIVE_MODE_ONLY);
		CASE_AND_STRING(AUDCLNT_E_BUFDURATION_PERIOD_NOT_EQUAL);
		CASE_AND_STRING(AUDCLNT_E_EVENTHANDLE_NOT_SET);
		CASE_AND_STRING(AUDCLNT_E_INCORRECT_BUFFER_SIZE);
		CASE_AND_STRING(AUDCLNT_E_BUFFER_SIZE_ERROR);
		CASE_AND_STRING(AUDCLNT_E_CPUUSAGE_EXCEEDED);
		CASE_AND_STRING(AUDCLNT_E_BUFFER_ERROR);
		CASE_AND_STRING(AUDCLNT_E_BUFFER_SIZE_NOT_ALIGNED);
		CASE_AND_STRING(AUDCLNT_E_INVALID_DEVICE_PERIOD);
		CASE_AND_STRING(AUDCLNT_E_INVALID_STREAM_FLAG);
		CASE_AND_STRING(AUDCLNT_E_ENDPOINT_OFFLOAD_NOT_CAPABLE);
		CASE_AND_STRING(AUDCLNT_E_OUT_OF_OFFLOAD_RESOURCES);
		CASE_AND_STRING(AUDCLNT_E_OFFLOAD_MODE_ONLY);
		CASE_AND_STRING(AUDCLNT_E_NONOFFLOAD_MODE_ONLY);
		CASE_AND_STRING(AUDCLNT_E_RESOURCES_INVALIDATED);
		CASE_AND_STRING(AUDCLNT_E_RAW_MODE_UNSUPPORTED);
		CASE_AND_STRING(AUDCLNT_E_ENGINE_PERIODICITY_LOCKED);
		CASE_AND_STRING(AUDCLNT_E_ENGINE_FORMAT_LOCKED);
		CASE_AND_STRING(AUDCLNT_S_BUFFER_EMPTY);
		CASE_AND_STRING(AUDCLNT_S_THREAD_ALREADY_REGISTERED);
		CASE_AND_STRING(AUDCLNT_S_POSITION_STALLED);

#undef CASE_AND_STRING

		case HRESULT(0xe000020b):						return TEXT("ERROR_NO_SUCH_DEVINST");
		case HRESULT(0x80010108):						return TEXT("RPC_E_DISCONNECTED");

		default:
		{
			// We don't know this error, ask this system if it does.
			TCHAR Buffer[1024] = { 0 };
			FString Msg(FPlatformMisc::GetSystemErrorMessage(Buffer, UE_ARRAY_COUNT(Buffer), InResult));

			// Anything to return? Otherwise "UNKNOWN"
			if (Msg.IsEmpty())
			{
				Msg = FString::Printf(TEXT("0x%x (UNKNOWN)"), InResult);
			}

			return Msg;
		}
		}
	}
}

