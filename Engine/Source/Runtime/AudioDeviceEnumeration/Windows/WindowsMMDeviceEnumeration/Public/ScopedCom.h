// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Windows/WindowsPlatformMisc.h"

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <combaseapi.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

namespace Audio
{
	struct FScopeComString final
	{
		LPTSTR StringPtr = nullptr;

		UE_NONCOPYABLE(FScopeComString)

		const LPTSTR Get() const
		{
			return StringPtr;
		}

		explicit operator bool() const
		{
			return Get() != nullptr;
		}

		FScopeComString(LPTSTR InStringPtr = nullptr)
			: StringPtr(InStringPtr)
		{}

		~FScopeComString()
		{
			if (StringPtr)
			{
				CoTaskMemFree(StringPtr);
			}
		}
	};

	class FScopedCoInitialize final
	{
	public:
		FScopedCoInitialize()
		{
#if PLATFORM_WINDOWS
			bCoInitialized = FPlatformMisc::CoInitialize(ECOMModel::Multithreaded);
#endif	
		}

		~FScopedCoInitialize()
		{
#if PLATFORM_WINDOWS
			if (bCoInitialized)
			{
				FPlatformMisc::CoUninitialize();
			}
#endif
		}
	private:
		bool bCoInitialized = false;
	};

}
