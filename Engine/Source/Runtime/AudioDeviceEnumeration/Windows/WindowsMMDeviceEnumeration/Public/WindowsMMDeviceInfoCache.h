// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#include "AudioMixer.h"
#include "ScopedCom.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"

#include "Microsoft/COMPointer.h"

THIRD_PARTY_INCLUDES_START
#include <mmreg.h>				// WAVEFORMATEX
#include <mmdeviceapi.h>		// IMMDeviceEnumerator
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformTypes.h"
#include "Windows/HideWindowsPlatformAtomics.h"

#define UE_API WINDOWSMMDEVICEENUMERATION_API

namespace Audio
{		
struct FWindowsMMDeviceCache : IAudioMixerDeviceChangedListener, IAudioPlatformDeviceInfoCache
{
	struct FCacheEntry
	{
		enum class UE_DEPRECATED(5.6, "EEndpointType is deprecated. Please use EDeviceEndpointType instead.") EEndpointType { Unknown, Render, Capture };
		
		FName DeviceId;							// Key
		FString FriendlyName;
		FString DeviceFriendlyName;
		EAudioDeviceState State = EAudioDeviceState::NotPresent;
		int32 NumChannels = 0;
		int32 SampleRate = 0;
		EDeviceEndpointType Type = EDeviceEndpointType::Unknown;
		uint32 ChannelBitmask = 0;				// Bitfield used to build output channels, for easy comparison.
		FName HardwareId;						// Unique string of the physical hardware device this MMDevice belongs to
		FString FilterId;						// Unique identifier for this device containing product id (pid), vendor id (vid), etc.

		TArray<EAudioMixerChannel::Type> OutputChannels;	// TODO. Generate this from the ChannelNum and bitmask when we are asked for it.
		mutable FRWLock MutationLock;

		FCacheEntry& operator=(const FCacheEntry& InOther);

		FCacheEntry& operator=(FCacheEntry&& InOther);

		FCacheEntry(const FCacheEntry& InOther);

		FCacheEntry(FCacheEntry&& InOther);

		FCacheEntry(const FString& InDeviceId);
	};

	TComPtr<IMMDeviceEnumerator> DeviceEnumerator;

	mutable FRWLock CacheMutationLock;							// R/W lock protects map and default arrays.
	TMap<FName, FCacheEntry> Cache;								// DeviceID GUID -> Info.
	FName DefaultCaptureId[(int32)EAudioDeviceRole::COUNT];		// Role -> DeviceID GUID
	FName DefaultRenderId[(int32)EAudioDeviceRole::COUNT];		// Role -> DeviceID GUID

	UE_API FWindowsMMDeviceCache();
	UE_API explicit FWindowsMMDeviceCache(bool bInEnableAggregateDeviceSupport);
	virtual ~FWindowsMMDeviceCache() = default;

	UE_API bool EnumerateChannelMask(uint32 InMask, FCacheEntry& OutInfo);

	UE_API bool EnumerateChannelFormat(const WAVEFORMATEX* InFormat, FCacheEntry& OutInfo);

	UE_API EDeviceEndpointType QueryDeviceDataFlow(const TComPtr<IMMDevice>& InDevice) const;

	UE_API bool EnumerateDeviceProps(const TComPtr<IMMDevice>& InDevice, FCacheEntry& OutInfo);

	UE_API bool EnumerateHardwareTopology(const TComPtr<IMMDevice>& InDevice, FCacheEntry& OutInfo);

	UE_DEPRECATED(5.8, "EnumerateEndpoints is no longer publicly callable; use Refresh() to rebuild the cache atomically.")
	UE_API void EnumerateEndpoints();

	UE_DEPRECATED(5.8, "EnumerateDefaults is no longer publicly callable; use Refresh() to rebuild the cache atomically.")
	UE_API void EnumerateDefaults();

	UE_API void OnDefaultCaptureDeviceChanged(const EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId) override;

	UE_API void OnDefaultRenderDeviceChanged(const EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId) override;

	UE_API void OnDeviceAdded(const FString& DeviceId, bool bIsRender) override;

	UE_API void OnDeviceRemoved(const FString& DeviceId, bool) override;

	UE_API TOptional<FCacheEntry> BuildCacheEntry(const FString& DeviceId);

	UE_API FString GetFriendlyName(FName InDeviceId) const;

	UE_API void OnDeviceStateChanged(const FString& DeviceId, const EAudioDeviceState InState, bool) override;

	UE_API void OnFormatChanged(const FString& InDeviceId, const FFormatChangedData& InFormat) override;

	UE_API void MakeDeviceInfo(const FCacheEntry& InEntry, FName InDefaultDevice, FAudioPlatformDeviceInfo& OutInfo) const;

	UE_API virtual TArray<FAudioPlatformDeviceInfo> GetAllActiveOutputDevices() const override;

	UE_API virtual bool IsAggregateHardwareDeviceId(const FName InDeviceID) const override;

	UE_API virtual TOptional<FAudioPlatformDeviceInfo> GetAggregateHardwareDeviceInfo(const FName InHardwareId, const EDeviceEndpointType InEndpointType) const;

	UE_API virtual TArray<FAudioPlatformDeviceInfo> GetLogicalAggregateDevices(const FName InHardwareId, const EDeviceEndpointType InEndpointType) const override;

	UE_API virtual TArray<FAudioPlatformDeviceInfo> SynthesizeAggregateDeviceList(const EDeviceEndpointType InType) const;

	UE_API virtual void Refresh() override;

	UE_API virtual void InvalidateDevice(FName InDeviceId) override;

	UE_API virtual FName GetDefaultOutputDevice(EAudioDeviceRole InRole) const override;

	UE_API FName GetDefaultOutputDevice_NoLock() const;

	UE_API TOptional<FAudioPlatformDeviceInfo> FindDefaultOutputDevice() const override;

	UE_API TOptional<FAudioPlatformDeviceInfo> FindActiveOutputDevice(FName InDeviceID) const override;

	UE_API bool IsAggregateDeviceSupportEnabled() const;

	static UE_API FString ExtractAggregateDeviceName(const FString& InName);
 
private:
	/** Re-enumerates all endpoints into the cache. Caller MUST hold CacheMutationLock for write. */
	UE_API void EnumerateEndpoints_NoLock();

	/** Re-reads default render/capture device IDs. Caller MUST hold CacheMutationLock for write. */
	UE_API void EnumerateDefaults_NoLock();

	struct FCacheKeyFuncs : BaseKeyFuncs<FWindowsMMDeviceCache::FCacheEntry, FName>
	{
		static KeyInitType GetSetKey(ElementInitType Element)
		{
			return Element.HardwareId;
		}

		static bool Matches(KeyInitType A, KeyInitType B)
		{
			return A.IsEqual(B, ENameCase::CaseSensitive);
		}

		static uint32 GetKeyHash(KeyInitType Key)
		{
			return GetTypeHash(Key);
		}
	};

	struct FDeviceChannelInfo
	{
		uint32 LogicDeviceChannelCount = 0;
		uint32 TotalChannelCount = 0;
	};

	bool bIsAggregateDeviceSupportEnabled = false;
	
	UE_API void GetHardwareInfo(TSet<FCacheEntry, FCacheKeyFuncs>& OutUniqueHardwareIds, TMap<FName, FDeviceChannelInfo>& OutDeviceChannelInfos, EDeviceEndpointType InType) const;

	static UE_API FAudioPlatformDeviceInfo CreateAggregateDeviceInfo(const FCacheEntry& InCacheEntry, const FDeviceChannelInfo& InDeviceChannelInfo);
	
	static UE_API int32 ExtractAggregateChannelNumber(const FString& InName);
};

}// namespace Audio

#undef UE_API
