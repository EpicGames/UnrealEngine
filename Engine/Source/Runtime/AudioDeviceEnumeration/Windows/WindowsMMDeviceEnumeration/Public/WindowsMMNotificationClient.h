// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#include <atomic>
#include "AudioMixer.h"	
#include "Misc/ScopeRWLock.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"

#include "Microsoft/COMPointer.h"

THIRD_PARTY_INCLUDES_START
#include <mmdeviceapi.h>			// IMMNotificationClient
#include <audiopolicy.h>			// IAudioSessionEvents
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"

namespace Audio
{
	class IAudioMixerDeviceChangedListener;

	class FWindowsMMNotificationClient final : public IMMNotificationClient
#if PLATFORM_WINDOWS
		, public IAudioSessionEvents
#endif //PLATFORM_WINDOWS
	{
	public:
		WINDOWSMMDEVICEENUMERATION_API FWindowsMMNotificationClient();
		~FWindowsMMNotificationClient();

		// TODO: Ideally we'd use the cache instead of ask for this.
		WINDOWSMMDEVICEENUMERATION_API bool IsRenderDevice(const FString& InDeviceId) const;

		WINDOWSMMDEVICEENUMERATION_API FString GetFriendlyName(const FString InDeviceID);
		WINDOWSMMDEVICEENUMERATION_API FString GetFriendlyName(const TComPtr<IMMDevice>& InDevice);

		WINDOWSMMDEVICEENUMERATION_API TComPtr<IMMDevice> GetDevice(const FString InDeviceID) const;
		WINDOWSMMDEVICEENUMERATION_API uint32 ReleaseClient();

		WINDOWSMMDEVICEENUMERATION_API void RegisterDeviceChangedListener(Audio::IAudioMixerDeviceChangedListener* DeviceChangedListener);
		WINDOWSMMDEVICEENUMERATION_API void UnRegisterDeviceDeviceChangedListener(Audio::IAudioMixerDeviceChangedListener* DeviceChangedListener);

		// Begin IUnknown overrides
		HRESULT STDMETHODCALLTYPE QueryInterface(const IID& IId, void** UnknownPtrPtr) override;
		ULONG STDMETHODCALLTYPE AddRef() override;
		ULONG STDMETHODCALLTYPE Release() override;
		// End IUnknown overrides
		
		// Begin IMMNotificationClient overrides
		HRESULT STDMETHODCALLTYPE OnDefaultDeviceChanged(EDataFlow InFlow, ERole InRole, LPCWSTR pwstrDeviceId) override;
		HRESULT STDMETHODCALLTYPE OnDeviceAdded(LPCWSTR pwstrDeviceId) override;;
		HRESULT STDMETHODCALLTYPE OnDeviceRemoved(LPCWSTR pwstrDeviceId) override;
		HRESULT STDMETHODCALLTYPE OnDeviceStateChanged(LPCWSTR pwstrDeviceId, DWORD dwNewState) override;
		HRESULT STDMETHODCALLTYPE OnPropertyValueChanged(LPCWSTR pwstrDeviceId, const PROPERTYKEY key);
		// End IMMNotificationClient overrides

#if PLATFORM_WINDOWS
		WINDOWSMMDEVICEENUMERATION_API bool RegisterForSessionNotifications(const TComPtr<IMMDevice>& InDevice);
		WINDOWSMMDEVICEENUMERATION_API bool RegisterForSessionNotifications(const FString& InDeviceId);
		WINDOWSMMDEVICEENUMERATION_API void UnregisterForSessionNotifications();

		// Begin IAudioSessionEvents overrides
		HRESULT STDMETHODCALLTYPE OnDisplayNameChanged(
			LPCWSTR NewDisplayName,
			LPCGUID EventContext) override;

		HRESULT STDMETHODCALLTYPE OnIconPathChanged(
			LPCWSTR NewIconPath,
			LPCGUID EventContext) override;

		HRESULT STDMETHODCALLTYPE OnSimpleVolumeChanged(
			float NewVolume,
			BOOL NewMute,
			LPCGUID EventContext) override;

		HRESULT STDMETHODCALLTYPE OnChannelVolumeChanged(
			DWORD ChannelCount,
			float NewChannelVolumeArray[],
			DWORD ChangedChannel,
			LPCGUID EventContext) override;

		HRESULT STDMETHODCALLTYPE OnGroupingParamChanged(
			LPCGUID NewGroupingParam,
			LPCGUID EventContext) override;

		HRESULT STDMETHODCALLTYPE OnStateChanged(
			AudioSessionState NewState) override;

		HRESULT STDMETHODCALLTYPE OnSessionDisconnected(
			AudioSessionDisconnectReason InDisconnectReason);
		// End IAudioSessionEvents overrides

	private:
		FCriticalSection SessionRegistrationCS;
		TComPtr<IAudioSessionManager> SessionManager;
		TComPtr<IAudioSessionControl> SessionControls;
		TComPtr<IMMDevice> DeviceListeningToSessionEvents;

		bool bComInitialized = false;
		std::atomic<bool> bHasDisconnectSessionHappened = false;

#endif //PLATFORM_WINDOWS
		
	private:
		LONG Ref = 1; // Start with self-reference
		TSet<Audio::IAudioMixerDeviceChangedListener*> Listeners;
		FRWLock ListenersSetRwLock;
		TComPtr<IMMDeviceEnumerator> DeviceEnumerator = nullptr;
	};
}// namespace Audio
