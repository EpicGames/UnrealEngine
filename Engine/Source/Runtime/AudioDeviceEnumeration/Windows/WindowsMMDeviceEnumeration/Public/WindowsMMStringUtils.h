// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"
#include "HAL/Platform.h"

THIRD_PARTY_INCLUDES_START
#include <intsafe.h>
#include "Windows/AllowWindowsPlatformTypes.h"

#include <mmdeviceapi.h>			// IMMNotificationClient
#include <audiopolicy.h>			// IAudioSessionEvents

#include "Windows/HideWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_END

namespace Audio
{
	WINDOWSMMDEVICEENUMERATION_API const TCHAR* ToString(AudioSessionDisconnectReason InDisconnectReason);
	WINDOWSMMDEVICEENUMERATION_API const TCHAR* ToString(ERole InRole);
	WINDOWSMMDEVICEENUMERATION_API const TCHAR* ToString(EDataFlow InFlow);
	WINDOWSMMDEVICEENUMERATION_API FString ToFString(const PROPERTYKEY Key);

	WINDOWSMMDEVICEENUMERATION_API FString ToFString(const TArray<EAudioMixerChannel::Type>& InChannels);

	WINDOWSMMDEVICEENUMERATION_API const TCHAR* ToString(EAudioDeviceRole InRole);
	WINDOWSMMDEVICEENUMERATION_API const TCHAR* ToString(EAudioDeviceState InState);
	WINDOWSMMDEVICEENUMERATION_API FString AudioClientErrorToFString(HRESULT InResult);
}
