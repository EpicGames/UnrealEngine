// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WindowsMMDeviceEnumeration : ModuleRules
{
	public WindowsMMDeviceEnumeration(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.Add("TargetPlatform");

		if (Target.bCompileAgainstEngine)
        {
            PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					 "Engine",
				}
			);
		}

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
            }
		);

		PrecompileForTargets = PrecompileTargetsType.None;

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrecompileForTargets = PrecompileTargetsType.Any;
		}
	}
}
